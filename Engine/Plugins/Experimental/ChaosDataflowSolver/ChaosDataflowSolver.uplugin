{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ChaosDataflowSolver",
	"Description": "",
	"Category": "Other",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosDataflowSolver",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}