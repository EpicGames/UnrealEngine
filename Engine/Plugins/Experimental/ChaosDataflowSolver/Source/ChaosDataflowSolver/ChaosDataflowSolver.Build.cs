// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosDataflowSolver : ModuleRules
{
	public ChaosDataflowSolver(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
			}
		);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
			}
		);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"DataflowCore",
				"DataflowEngine",
				"DataflowSimulation",
				"RigidPhysics",
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"Landscape",
			}
		);

		SetupModulePhysicsSupport(Target);
		PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");


		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
		);
	}
}
