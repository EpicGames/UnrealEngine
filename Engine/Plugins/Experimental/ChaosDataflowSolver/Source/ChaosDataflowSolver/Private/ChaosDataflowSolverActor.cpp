// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosDataflowSolverActor.h"
#include "UObject/ConstructorHelpers.h"
#include "PhysicsSolver.h"
#include "ChaosModule.h"

#include "Components/BillboardComponent.h"
#include "EngineUtils.h"
#include "ChaosSolversModule.h"
#include "Chaos/Framework/DebugSubstep.h"
#include "Engine/Texture2D.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Dataflow/DataflowSimulationNodes.h"

#include "RigidPhysics//RigidFwd.h"
#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidModifier.h"
#include "RigidPhysics/RigidPhysicsService.h"
#include "RigidPhysics/RigidScene.h"
#include "RigidAssetUserData.h"

#include "RigidComponents/PrimitiveRigidComponentInterface.h"
#include "RigidComponents/InstancedRigidComponentInterface.h"
#include "RigidComponents/LandscapeRigidComponentInterface.h"
#include "RigidComponents/SkeletalRigidComponentInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosDataflowSolverActor)

AChaosDataflowSolverActor::AChaosDataflowSolverActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bTickEvenWhenPaused = false;
	PrimaryActorTick.bStartWithTickEnabled = true;
	PrimaryActorTick.TickGroup = TG_PrePhysics;
}

void AChaosDataflowSolverActor::BeginPlay()
{
	Super::BeginPlay();
}

void AChaosDataflowSolverActor::EndPlay(const EEndPlayReason::Type ReasonEnd)
{
	Super::EndPlay(ReasonEnd);

#if UE_RIGIDPHYSICS_API_ENABLED 
	if (SolverContext)
	{
		if (FRigidPhysicsSceneDataflowState* State = SolverContext->GetRigidState())
		{
			if (UE::Physics::FRigidContextGameRW Context = State->Handle.LockRW())
			{
				for (TWeakObjectPtr<UPrimitiveComponent> WeakComponent : RegisteredComponents)
				{
					if (UPrimitiveComponent* Component = WeakComponent.Get())
					{
						URigidAssetUserData* RigidData = Component->GetAssetUserData<URigidAssetUserData>();
						if (RigidData)
						{
							for (UE::Physics::FRigidBodyHandle& Handle : RigidData->Bodies)
							{
								Context->DestroyBody(Handle.Pin(Context));
							}
						}

						Component->RemoveUserDataOfClass(URigidAssetUserData::StaticClass());
					}
				}

				RegisteredComponents.Empty();
			}

			UE::Physics::FRigidPhysicsService& Service = UE::Physics::FRigidPhysicsService::GetInstance();
			Service.DestroyScene(State->Handle);
		}
	}
#endif

}

void AChaosDataflowSolverActor::BuildSimulationProxy()
{

}
void AChaosDataflowSolverActor::ResetSimulationProxy()
{

}
void AChaosDataflowSolverActor::WriteToSimulation(const float DeltaTime, const bool bAsyncTask)
{
}
void AChaosDataflowSolverActor::ReadFromSimulation(const float DeltaTime, const bool bAsyncTask)
{
	
}

void AChaosDataflowSolverActor::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

#if UE_RIGIDPHYSICS_API_ENABLED 
	if (SolverContext && SimulationAsset.DataflowAsset)
	{
		if (FRigidPhysicsSceneDataflowState* State = SolverContext->GetRigidState())
		{
			if (UE::Physics::FRigidContextGameRW Context = State->Handle.LockRW())
			{
				for (TWeakObjectPtr<UPrimitiveComponent> WeakComponent: RegisteredComponents)
				{
					if (UPrimitiveComponent* Component = WeakComponent.Get())
					{
						// TODO: Edit logic later when Landscapes and Skeletal are fully implemented
						bool bBasicType = !Component->IsA<UInstancedStaticMeshComponent>() && !Component->IsA<ULandscapeHeightfieldCollisionComponent>() && !Component->IsA<USkeletalMeshComponent>();

						URigidAssetUserData* RigidData = Component->GetAssetUserData<URigidAssetUserData>();
						if (bBasicType && RigidData && RigidData->Bodies.Num() > 0)
						{
							if (UE::Physics::TRigidBodyPtr<UE::Physics::FRigidContextGameRW> BodyPtr = RigidData->Bodies[0].Pin(Context))
							{
								FTransform Transform = Component->GetComponentToWorld();

								if (!BodyPtr->GetTransform().EqualsNoScale(Transform))
								{
									if (BodyPtr->IsKinematic())
									{
										BodyPtr->SetKinematicTarget(Transform);
									}
									else
									{
										BodyPtr->UpdateTransform(Transform);
									}
								}
							}
						}
					}
				}
			}

			
			if (const TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = SimulationAsset.DataflowAsset->GetDataflowGraph())
			{
				const TArray<UE::Dataflow::FGraph::FNodeSharedPtr>& Array = DataflowGraph->GetFilteredNodes(FSceneSimulationTerminalDataflowNode::StaticType());

				SolverContext->SetTimingInfos(DeltaTime, 0);
				SolverContext->ClearAllData();

				for (const TSharedPtr<FDataflowNode>& ExecutionNode : Array)
				{
					SolverContext->Evaluate(ExecutionNode.Get(), nullptr);
				}
			}

			if (UE::Physics::FRigidContextGameRO Context = State->Handle.LockRO())
			{
				for (TWeakObjectPtr<UPrimitiveComponent> WeakComponent : RegisteredComponents)
				{
					if (UPrimitiveComponent* Component = WeakComponent.Get())
					{
						if (URigidAssetUserData* RigidData = Component->GetAssetUserData<URigidAssetUserData>())
						{
							if (UInstancedStaticMeshComponent* InstComponent = Cast<UInstancedStaticMeshComponent>(Component))
							{
							}
							else if (ULandscapeHeightfieldCollisionComponent* LandscapeComponent = Cast<ULandscapeHeightfieldCollisionComponent>(Component))
							{
							}
							else if (USkeletalMeshComponent* SkelComponent = Cast<USkeletalMeshComponent>(Component))
							{
								SkeletalRigidComponentInterface::OnSolverEndFrame(SkelComponent, State->Handle, RigidData->Bodies);
							}
							else
							{
								PrimitiveRigidComponentInterface::OnSolverEndFrame(Component, State->Handle, RigidData->Bodies);
							}
						}
					}
				}
			}
		}
	}
#endif

}

void AChaosDataflowSolverActor::SetSolverActive(bool bActive)
{

}

void AChaosDataflowSolverActor::PostInitializeComponents()
{
	Super::PostInitializeComponents();

#if UE_RIGIDPHYSICS_API_ENABLED 
	if (SimulationAsset.DataflowAsset) {
		
		const TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = SimulationAsset.DataflowAsset->GetDataflowGraph();

		if (DataflowGraph)
		{
			SolverContext = MakeUnique<UE::Dataflow::FDataflowSimulationContext>(SimulationAsset.DataflowAsset);

			SolverContext->SetRigidSceneName(GetName());

			const TArray<UE::Dataflow::FGraph::FNodeSharedPtr>& Array = DataflowGraph->GetFilteredNodes(FRigidSceneSetupTerminalDataflowNode::StaticType());

			if(!Array.IsEmpty())
			{
				for (const TSharedPtr<FDataflowNode>& ExecutionNode : Array)
				{
					SolverContext->Evaluate(ExecutionNode.Get(), nullptr);
				}
			}
		}
	}
#endif

}

void AChaosDataflowSolverActor::RegisterPhysicsComponent(UPrimitiveComponent* Component)
{

#if UE_RIGIDPHYSICS_API_ENABLED 
	if (!Component || !SolverContext)
	{
		return;
	}

	if (FRigidPhysicsSceneDataflowState* State = SolverContext->GetRigidState())
	{
		URigidAssetUserData* Data = NewObject<URigidAssetUserData>(Component, NAME_None, RF_Transient);

		if (UInstancedStaticMeshComponent* InstComponent = Cast<UInstancedStaticMeshComponent>(Component))
		{
			InstancedRigidComponentInterface::OnCreateSolverBodies(InstComponent, State->Handle, Data->Bodies);
		}
		else if (ULandscapeHeightfieldCollisionComponent* LandscapeComponent = Cast<ULandscapeHeightfieldCollisionComponent>(Component))
		{
		}
		else if (USkeletalMeshComponent* SkelComponent = Cast<USkeletalMeshComponent>(Component))
		{
			SkeletalRigidComponentInterface::OnCreateSolverBodies(SkelComponent, State->Handle, Data->Bodies);
		}
		else
		{
			PrimitiveRigidComponentInterface::OnCreateSolverBodies(Component, State->Handle, Data->Bodies);
		}

		Component->AddAssetUserData(Data);

		RegisteredComponents.Add(Component);
	}
#endif

}

void AChaosDataflowSolverActor::UnregisterPhysicsComponent(UPrimitiveComponent* Component)
{

#if UE_RIGIDPHYSICS_API_ENABLED 
	if (!Component || !SolverContext)
	{
		return;
	}

	int32 Index = RegisteredComponents.Find(Component);
	
	if (Index != INDEX_NONE)
	{
		if (FRigidPhysicsSceneDataflowState* State = SolverContext->GetRigidState())
		{
			if (UE::Physics::FRigidContextGameRW Context = State->Handle.LockRW())
			{
				URigidAssetUserData* RigidData = Component->GetAssetUserData<URigidAssetUserData>();
				if (RigidData)
				{
					for (UE::Physics::FRigidBodyHandle& Handle : RigidData->Bodies)
					{
						Context->DestroyBody(Handle.Pin(Context));
					}
				}

				Component->RemoveUserDataOfClass(URigidAssetUserData::StaticClass());
				RegisteredComponents.RemoveAtSwap(Index);
			}
		}
	}
#endif

}




