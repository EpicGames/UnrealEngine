// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSolverBindingComponent.h"
#include "Chaos/DebugDrawQueue.h"
#include "ChaosLog.h"
#include "Debug/DebugDrawService.h"
#include "DrawDebugHelpers.h"
#if WITH_EDITOR
#include "UObject/UObjectIterator.h"
#endif
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSolverBindingComponent)

UChaosSolverBindingComponent::UChaosSolverBindingComponent()
{
}

void UChaosSolverBindingComponent::BeginDestroy()
{
	Super::BeginDestroy();
}

void UChaosSolverBindingComponent::BeginPlay()
{
	Super::BeginPlay();
	if (AActor* Owner = GetOwner())
	{
		TSet<UActorComponent*> Components = Owner->GetComponents();
		
		for (UActorComponent* Component : Components)
		{
			if (TObjectPtr<UPrimitiveComponent> SimulatingComponent = Cast<UPrimitiveComponent>(Component))
			{				
				if (SimulationActor)
				{
					SimulationActor->RegisterPhysicsComponent(SimulatingComponent);

					if (!bKeepKinematicInOriginal)
					{
						Component->DestroyPhysicsState();
					}
				}
			}
		}
	}

}

void UChaosSolverBindingComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	if (AActor* Owner = GetOwner())
	{
		TSet<UActorComponent*> Components = Owner->GetComponents();
		
		for (UActorComponent* Component : Components)
		{
			if (TObjectPtr<UPrimitiveComponent> SimulatingComponent = Cast<UPrimitiveComponent>(Component))
			{				
				if (SimulationActor)
				{
					SimulationActor->UnregisterPhysicsComponent(SimulatingComponent);
				}
			}
		}
	}
}

#if WITH_EDITOR
void UChaosSolverBindingComponent::OnRegister()
{
	Super::OnRegister();
}

void UChaosSolverBindingComponent::OnUnregister()
{
	Super::OnUnregister();
}
#endif // WITH_EDITOR

void UChaosSolverBindingComponent::BindWorldDelegates()
{
}

void UChaosSolverBindingComponent::HandlePostWorldInitialization(UWorld* World, const UWorld::InitializationValues)
{
}





