// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Components/InstancedStaticMeshComponent.h"

#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidModifier.h"
#include "RigidPhysics/RigidScene.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/BodySetup.h"
#include "Physics/Experimental/ChaosInterfaceUtils.h"

class InstancedRigidComponentInterface
{
	public:
	static void OnCreateSolverBodies(UInstancedStaticMeshComponent* Component, UE::Physics::FRigidSceneHandle& SceneHandle, TArray<UE::Physics::FRigidBodyHandle>& OutBodies)
	{
		using namespace Chaos;
		AActor* Owner = Component->GetOwner();

		const TArray<FBodyInstance*>& OriginalBodies = Component->GetInstanceBodies();
	
		for (int i = 0; i < OriginalBodies.Num(); i++)
		{
			if (const FBodyInstance* OriginalBody = OriginalBodies[i])
			{
				if (UBodySetup* BodySetup = OriginalBody->GetBodySetup())
				{
					if (UE::Physics::FRigidContextGameRW Context = SceneHandle.LockRW())
					{
						UE::Physics::FRigidBodyHandle Body = Context->CreateBody(UE::Physics::FRigidDebugName(OriginalBody->GetBodyDebugName()), UE::Physics::ERigidMovementType::Static);

						if (UE::Physics::TRigidBodyPtr<UE::Physics::FRigidContextGameRW> BodyPtr = Body.Pin(Context))
						{
							FTransform Transform = FTransform::Identity;

							Component->GetInstanceTransform(i, Transform, true);

							FBodyCollisionData BodyCollisionData;
							OriginalBody->BuildBodyFilterData(BodyCollisionData.CollisionFilterData);
							FBodyInstance::BuildBodyCollisionFlags(BodyCollisionData.CollisionFlags, OriginalBody->GetCollisionEnabled(), BodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple);

							TArray<FPhysicalMaterialMaskParams> OutPhysMaterialMasks;

							FGeometryAddParams AddParams;
							AddParams.bDoubleSided = OriginalBody->GetBodySetup()->bDoubleSidedGeometry;
							AddParams.CollisionData = BodyCollisionData; 
							AddParams.CollisionTraceType = BodySetup->GetCollisionTraceFlag();
							AddParams.SimpleMaterial = OriginalBody->GetSimplePhysicalMaterial();
							AddParams.ComplexMaterials = TArrayView<UPhysicalMaterial*>(OriginalBody->GetComplexPhysicalMaterials(OutPhysMaterialMasks));
							AddParams.ComplexMaterialMasks = TArrayView<FPhysicalMaterialMaskParams>(OutPhysMaterialMasks);
							AddParams.Scale = Transform.GetScale3D();
							AddParams.LocalTransform = FTransform::Identity;
							AddParams.WorldTransform = Transform;
							AddParams.Geometry = &BodySetup->AggGeom;
							AddParams.TriMeshGeometries = MakeArrayView(BodySetup->TriMeshGeometries);

							TArray<Chaos::FImplicitObjectPtr> Geoms;
							Chaos::FShapesArray Shapes;
							ChaosInterface::CreateGeometry(AddParams, Geoms, Shapes);

							const int32 NumShapes = Shapes.Num();
		
							for (int32 ShapeIndex = 0; ShapeIndex < NumShapes; ShapeIndex++)
							{
								BodyPtr->CreateShape(MoveTemp(Shapes[ShapeIndex]));
							}

							BodyPtr->InitTransform(Transform);

							BodyPtr->SetMass(OriginalBody->GetBodyMass());
							BodyPtr->SetInertia(OriginalBody->GetBodyInertiaTensor() * OriginalBody->InertiaTensorScale);

							BodyPtr->Activate();
						}

						OutBodies.Add(Body);
					}
				}
			}
		}
	}
};

#endif