// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "LandscapeHeightfieldCollisionComponent.h"

#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidModifier.h"
#include "RigidPhysics/RigidScene.h"

class LandscapeRigidComponentInterface
{

};

#endif