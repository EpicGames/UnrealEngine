// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Components/PrimitiveComponent.h"

#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidModifier.h"
#include "RigidPhysics/RigidScene.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/BodySetup.h"
#include "Physics/Experimental/ChaosInterfaceUtils.h"

class PrimitiveRigidComponentInterface
{
	public:
	static void OnCreateSolverBodies(UPrimitiveComponent* Component, UE::Physics::FRigidSceneHandle& SceneHandle, TArray<UE::Physics::FRigidBodyHandle>& OutBodies)
	{
		using namespace Chaos;
		AActor* Owner = Component->GetOwner();

		if (FBodyInstance* OriginalBody = Component->GetBodyInstance())
		{
			if (UBodySetup* BodySetup = OriginalBody->GetBodySetup())
			{
				if (UE::Physics::FRigidContextGameRW Context = SceneHandle.LockRW())
				{
					UE::Physics::ERigidMovementType MovementType;

					if (!OriginalBody->IsNonKinematic())
					{
						MovementType = UE::Physics::ERigidMovementType::Kinematic;
					}
					else
					{
						MovementType = UE::Physics::ERigidMovementType::Dynamic;
					}

					UE::Physics::FRigidBodyHandle Body = Context->CreateBody(UE::Physics::FRigidDebugName(Component->GetName()), MovementType);

					if (UE::Physics::TRigidBodyPtr<UE::Physics::FRigidContextGameRW> BodyPtr = Body.Pin(Context))
					{
						FTransform Transform = OriginalBody->GetUnrealWorldTransform();

						FBodyCollisionData BodyCollisionData;
						OriginalBody->BuildBodyFilterData(BodyCollisionData.CollisionFilterData);
						FBodyInstance::BuildBodyCollisionFlags(BodyCollisionData.CollisionFlags, OriginalBody->GetCollisionEnabled(), BodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple);

						TArray<FPhysicalMaterialMaskParams> OutPhysMaterialMasks;

						FGeometryAddParams AddParams;
						AddParams.bDoubleSided = OriginalBody->GetBodySetup()->bDoubleSidedGeometry;
						AddParams.CollisionData = BodyCollisionData;
						AddParams.CollisionTraceType = BodySetup->GetCollisionTraceFlag();
						AddParams.SimpleMaterial = OriginalBody->GetSimplePhysicalMaterial();
						AddParams.ComplexMaterials = TArrayView<UPhysicalMaterial*>(OriginalBody->GetComplexPhysicalMaterials(OutPhysMaterialMasks));
						AddParams.ComplexMaterialMasks = TArrayView<FPhysicalMaterialMaskParams>(OutPhysMaterialMasks);
						AddParams.Scale = Component->GetComponentScale();
						AddParams.LocalTransform = FTransform::Identity;
						AddParams.WorldTransform = Transform;
						AddParams.Geometry = &BodySetup->AggGeom;
						AddParams.TriMeshGeometries = MakeArrayView(BodySetup->TriMeshGeometries);

						TArray<Chaos::FImplicitObjectPtr> Geoms;
						Chaos::FShapesArray Shapes;
						ChaosInterface::CreateGeometry(AddParams, Geoms, Shapes);

						const int32 NumShapes = Shapes.Num();

						for (int32 ShapeIndex = 0; ShapeIndex < NumShapes; ++ShapeIndex)
						{
							BodyPtr->CreateShape(MoveTemp(Shapes[ShapeIndex])); 
						}

						Transform.SetScale3D(FVector(1.0f, 1.0f, 1.0f));
						BodyPtr->InitTransform(Transform);

						BodyPtr->SetMass(OriginalBody->GetBodyMass());
						BodyPtr->SetInertia(OriginalBody->GetBodyInertiaTensor() * OriginalBody->InertiaTensorScale);

						BodyPtr->Activate();
					}

					OutBodies.Add(Body);

					OriginalBody->SetInstanceSimulatePhysics(false);
				}
			}
		}
	}

	static void OnSolverEndFrame(UPrimitiveComponent* Component, UE::Physics::FRigidSceneHandle& SceneHandle, TArray<UE::Physics::FRigidBodyHandle>& Bodies)
	{
		if (Bodies.IsEmpty())
		{
			return;
		}

		//Only 1 body;
		UE::Physics::FRigidBodyHandle BodyHandle = Bodies[0];

		if (UE::Physics::FRigidContextGameRO Context = SceneHandle.LockRO())
		{
			if (UE::Physics::TRigidBodyPtr<UE::Physics::FRigidContextGameRO> BodyPtr = BodyHandle.Pin(Context))
			{
				if (!BodyPtr->IsStatic())
				{
					if (!BodyPtr->IsKinematic())
					{
						FTransform NewTransform = BodyPtr->GetTransform();

						const FVector DesiredDelta = FTransform::SubtractTranslations(NewTransform, Component->GetComponentTransform());

						Component->MoveComponent(DesiredDelta, NewTransform.GetRotation(), false, NULL, MOVECOMP_SkipPhysicsMove);
					}
				}
			}
		}
	}
};

#endif