// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SkeletalMeshComponent.h"

#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidModifier.h"
#include "RigidPhysics/RigidScene.h"

#if UE_RIGIDPHYSICS_API_ENABLED

class SkeletalRigidComponentInterface
{
	public:
	static void OnCreateSolverBodies(USkeletalMeshComponent* Component, UE::Physics::FRigidSceneHandle& SceneHandle, TArray<UE::Physics::FRigidBodyHandle>& OutBodies)
	{
	}

	static void OnSolverEndFrame(USkeletalMeshComponent* Component, UE::Physics::FRigidSceneHandle& SceneHandle, TArray<UE::Physics::FRigidBodyHandle>& Bodies)
	{
	}
};

#endif