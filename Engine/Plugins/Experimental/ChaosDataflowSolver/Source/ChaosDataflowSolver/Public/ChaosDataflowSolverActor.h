// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Dataflow/Interfaces/DataflowPhysicsSolver.h"
#include "Dataflow/DataflowSimulationInterface.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "RigidPhysics/RigidFactory.h"
#include "RigidPhysics/RigidScene.h"
#include "RigidPhysics/RigidBody.h"
#include "Dataflow/RigidPhysicsSceneState.h"
#include "Dataflow/DataflowSimulationManager.h"

#include "Components/InstancedStaticMeshComponent.h"

#include "ChaosDataflowSolverActor.generated.h"

struct FDataflowSimulationAsset;
class UChaosSolverBindingComponent;

UCLASS(meta=(Experimental), MinimalAPI)
class AChaosDataflowSolverActor : public AActor, public IDataflowPhysicsSolverInterface
{
	GENERATED_UCLASS_BODY()

public:
	virtual void PostInitializeComponents() override;
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type ReasonEnd) override;
	/** Controls whether the solver is able to simulate particles it controls */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	virtual void SetSolverActive(bool bActive);

	//~ Begin IDataflowPhysicsSolverInterface interface
	virtual FString GetSimulationName() const override {return GetName();};
	virtual FDataflowSimulationAsset& GetSimulationAsset() override {return SimulationAsset;};
	virtual const FDataflowSimulationAsset& GetSimulationAsset() const override {return SimulationAsset;};
	virtual FDataflowSimulationProxy* GetSimulationProxy() override { return nullptr; }
	virtual const FDataflowSimulationProxy* GetSimulationProxy() const  override { return nullptr; }
	virtual void BuildSimulationProxy() override;
    virtual void ResetSimulationProxy() override;
	virtual void WriteToSimulation(const float DeltaTime, const bool bAsyncTask) override;
	virtual void ReadFromSimulation(const float DeltaTime, const bool bAsyncTask) override;
	virtual void Tick(float DeltaTime) override;

	void RegisterPhysicsComponent(UPrimitiveComponent* Component);
	void UnregisterPhysicsComponent(UPrimitiveComponent* Component);

private:
	/* Solver dataflow asset used to advance in time */
	UPROPERTY(EditAnywhere, Category = "Physics", meta=(EditConditionHides))
	FDataflowSimulationAsset SimulationAsset;

	TArray<TWeakObjectPtr<UPrimitiveComponent>> RegisteredComponents;

	TUniquePtr<UE::Dataflow::FDataflowSimulationContext> SolverContext;
};