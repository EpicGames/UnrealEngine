// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "ChaosDataflowSolverActor.h"
#include "Chaos/ChaosDebugDrawDeclares.h"
#include "Chaos/Declares.h"
#include "Chaos/DebugDrawQueue.h"
#include "Engine/World.h"
#include "ChaosSolverBindingComponent.generated.h"

UCLASS(BlueprintType, ClassGroup = Chaos, meta = (Experimental, BlueprintSpawnableComponent), MinimalAPI)
class UChaosSolverBindingComponent: public UActorComponent
{
	GENERATED_BODY()

public:

	UChaosSolverBindingComponent();

	//~ Begin UActorComponent interface
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	virtual void BeginDestroy() override;
	//~ End UActorComponent interface

	static void BindWorldDelegates();

protected:
#if WITH_EDITOR
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
#endif // WITH_EDITOR

private:
	static void HandlePostWorldInitialization(UWorld* World, const UWorld::InitializationValues IVS);

	UPROPERTY(EditAnywhere, Category = "Physics")
	TSoftObjectPtr<AChaosDataflowSolverActor> SimulationActor;

	UPROPERTY(EditAnywhere, Category = "Physics")
	bool bKeepKinematicInOriginal;
};

