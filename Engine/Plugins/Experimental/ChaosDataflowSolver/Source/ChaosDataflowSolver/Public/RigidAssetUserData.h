// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "CoreMinimal.h"

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED
#include "RigidPhysics/RigidBody.h"
#endif

#include "RigidAssetUserData.generated.h"


UCLASS(meta=(Experimental), MinimalAPI, NotBlueprintable, HideCategories = (Object))
class URigidAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

#if UE_RIGIDPHYSICS_API_ENABLED
public:
	TArray<UE::Physics::FRigidBodyHandle> Bodies;
#endif

};